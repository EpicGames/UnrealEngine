{
	"FileVersion" : 3,
	"bIsPluginExtension": true,
	"Modules" :
	[
		{
			"Name" : "ScreenReader",
			"Type" : "Runtime",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList":
			[
				"WinGDK"
			]
		}
	]
}