{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "ScreenReader",
	"Description" : "A plugin that contains accessibility classes and frameworks that can be extended to offer vision accessibility services.",
	"Category" : "Accessibility",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsExperimental" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ScreenReader",
			"Type" : "Runtime",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" :
			[
				"Win64",
				"Mac",
				"Linux",
				"LinuxArm64"
			],
			"TargetDenyList" :
			[
				"Server"
			]
		}
	],
	"Plugins" :
	[
		{
			"Name" : "TextToSpeech",
			"Enabled": true
		}
	]
}
