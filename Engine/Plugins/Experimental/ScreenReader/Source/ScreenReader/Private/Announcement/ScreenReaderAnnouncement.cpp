// Copyright Epic Games, Inc. All Rights Reserved.

#include "Announcement/ScreenReaderAnnouncement.h"
#include "ScreenReaderLog.h"

FScreenReaderAnnouncementInfo::FScreenReaderAnnouncementInfo(bool bInShouldQueue, bool bInInterruptable, EScreenReaderAnnouncementPriority InPriority)
	: bShouldQueue( bInShouldQueue)
	, bInterruptable(bInInterruptable)
	, Priority(InPriority)
	, Timestamp(FPlatformTime::Seconds())
{

}

FString FScreenReaderAnnouncementInfo::ToString() const
{
	TStringBuilder<128> Builder;
	Builder.Append(TEXT("Announcement info: Should queue = "));
	const FString ShouldQueueValue = ShouldQueue() ? "true" : "false";
	Builder.Append(ShouldQueueValue);
	Builder.Append(TEXT(". Is interruptable = "));
	const FString IsInterruptableValue = IsInterruptable() ? "true" : "false";
	Builder.Append(IsInterruptableValue);
	Builder.Append(TEXT(". Priority = "));
	switch (GetPriority())
	{
	case EScreenReaderAnnouncementPriority::High:
		Builder.Append(TEXT("High."));
		break;
	case EScreenReaderAnnouncementPriority::Medium:
		Builder.Append(TEXT("Medium"));
		break;
	case EScreenReaderAnnouncementPriority::Low:
		Builder.Append(TEXT("Low"));
		break;
	default:
		UE_LOGF(LogScreenReaderAnnouncement, Error, "Unknown priorityannouncement priority level.");
		break;
	}
	Builder.Append(TEXT(". Timestamp = "));


	return Builder.ToString();
}

FScreenReaderAnnouncement::FScreenReaderAnnouncement(FString InAnnouncementString, FScreenReaderAnnouncementInfo InAnnouncementInfo)
	: AnnouncementString(MoveTemp(InAnnouncementString))
	, AnnouncementInfo(MoveTemp(InAnnouncementInfo))
{

}

FString FScreenReaderAnnouncement::ToString() const
{
	TStringBuilder<256> Builder;
	Builder.Append(TEXT("AnnouncementString = "));
	Builder.Append(GetAnnouncementString());
	Builder.Append(TEXT(". \n"));
	Builder.Append(GetAnnouncementInfo().ToString());
	return Builder.ToString();
}
