// Copyright Epic Games, Inc. All Rights Reserved.

#include "Announcement/ScreenReaderAnnouncementChannel.h"
#include "GenericPlatform/ScreenReaderReply.h"
#include "ScreenReaderLog.h"
#include "Announcement/ScreenReaderWidgetAnnouncementBuilder.h"
#include "GenericPlatform/TextToSpeechBase.h"

namespace ScreenReaderAnnouncementChannel_Private
{
	static FScreenReaderWidgetAnnouncementBuilder WidgetAnnouncementBuilder;
}

FScreenReaderAnnouncementChannel::FScreenReaderAnnouncementChannel(const TSharedRef<FTextToSpeechBase>& InTextToSpeech)
	: TextToSpeech(InTextToSpeech)
{
	InTextToSpeech->SetTextToSpeechFinishedSpeakingDelegate(FTextToSpeechBase::FOnTextToSpeechFinishSpeaking::CreateRaw(this, &FScreenReaderAnnouncementChannel::OnTextToSpeechFinishSpeaking));
}

FScreenReaderAnnouncementChannel::~FScreenReaderAnnouncementChannel()
{
	Deactivate();
}

void FScreenReaderAnnouncementChannel::Activate()
{
	ClearAllAnnouncements();
	TextToSpeech->Activate();
}

void FScreenReaderAnnouncementChannel::Deactivate()
{
	TextToSpeech->Deactivate();
	ClearAllAnnouncements();
}

FScreenReaderReply FScreenReaderAnnouncementChannel::RequestSpeak(FScreenReaderAnnouncement InAnnouncement)
{
	if (InAnnouncement.GetAnnouncementString().IsEmpty() || !TextToSpeech->IsActive())
	{
		return FScreenReaderReply::Unhandled();
	}
	
	UE_LOGF(LogScreenReaderAnnouncement, VeryVerbose, "Requesting to speak announcement %ls.", *InAnnouncement.ToString());
	if (IsSpeaking())
	{
		ensureMsgf(!CurrentAnnouncement.GetAnnouncementString().IsEmpty(), TEXT("Channel is Current announcement should have a valid string if announcement is being spoken. Check this algorithm."));
		UE_LOGF(LogScreenReaderAnnouncement, VeryVerbose, "Announcement currently being spoken.");
		// If the current announcement is currently being spoken,
		// We see if it can be interrupted by the passed in message 
		if (FScreenReaderAnnouncement::CanBeInterruptedBy(CurrentAnnouncement, InAnnouncement, FScreenReaderAnnouncement::FDefaultInterruptionPolicy()))
		{
			if (CurrentAnnouncement.GetAnnouncementInfo().ShouldQueue())
			{
				AnnouncementQueue.HeapPush(MoveTemp(CurrentAnnouncement), FScreenReaderAnnouncement::FDefaultComparator());
			}
			CurrentAnnouncement = MoveTemp(InAnnouncement);
			TextToSpeech->Speak(CurrentAnnouncement.GetAnnouncementString());
		}
		else if (InAnnouncement.GetAnnouncementInfo().ShouldQueue())
		{
			AnnouncementQueue.HeapPush(MoveTemp(InAnnouncement), FScreenReaderAnnouncement::FDefaultComparator());
		}
	}
	else
	{
		ensureMsgf(CurrentAnnouncement.GetAnnouncementString().IsEmpty(), TEXT("No announcement is being spoken, but current announcement has a valid string. Check your algorithm."));
		UE_LOGF(LogScreenReaderAnnouncement, VeryVerbose, "Nothing being spoken. Speaking immediately.");
		// nothing beimng spoken, speak passed in announcement immediately 
		CurrentAnnouncement = MoveTemp(InAnnouncement);
		TextToSpeech->Speak(CurrentAnnouncement.GetAnnouncementString());
	}
	return FScreenReaderReply::Handled();
}

FScreenReaderReply FScreenReaderAnnouncementChannel::StopSpeaking()
{
	TextToSpeech->StopSpeaking();
	return FScreenReaderReply::Handled();
}

bool FScreenReaderAnnouncementChannel::IsSpeaking() const
{
	return TextToSpeech->IsSpeaking();
}

FScreenReaderReply FScreenReaderAnnouncementChannel::ClearAllAnnouncements()
{
	CurrentAnnouncement = FScreenReaderAnnouncement();
	AnnouncementQueue.Empty();
	ensure(CurrentAnnouncement.GetAnnouncementString().IsEmpty());
	ensure(AnnouncementQueue.Num() == 0);
	return FScreenReaderReply::Handled();
}

FScreenReaderReply FScreenReaderAnnouncementChannel::RequestSpeakWidget(const TSharedRef<IAccessibleWidget>& InWidget)
{
	FString WidgetAnnouncement = ScreenReaderAnnouncementChannel_Private::WidgetAnnouncementBuilder.BuildWidgetAnnouncement(InWidget);
	return RequestSpeak(FScreenReaderAnnouncement(WidgetAnnouncement, FScreenReaderAnnouncementInfo::DefaultWidgetAnnouncement()));
}

float FScreenReaderAnnouncementChannel::GetSpeechVolume() const
{
	return TextToSpeech->GetVolume();
}

FScreenReaderReply FScreenReaderAnnouncementChannel::SetSpeechVolume(float InVolume)
{
	// FTextToSpeech API already takes care of range check and clamping of values 
	TextToSpeech->SetVolume(InVolume);
	return FScreenReaderReply::Handled();
}

float FScreenReaderAnnouncementChannel::GetSpeechRate() const
{
	return TextToSpeech->GetRate();
}

FScreenReaderReply FScreenReaderAnnouncementChannel::SetSpeechRate(float InRate)
{
	// The FTextToSpeech API already handles range checks and clamping for values  
	TextToSpeech->SetRate(InRate);
	return FScreenReaderReply::Handled();
}

FScreenReaderReply FScreenReaderAnnouncementChannel::MuteSpeech()
{
	TextToSpeech->Mute();
	return FScreenReaderReply::Handled();
}

FScreenReaderReply FScreenReaderAnnouncementChannel::UnmuteSpeech()
{
	TextToSpeech->Unmute();
	return FScreenReaderReply::Handled();
}

bool FScreenReaderAnnouncementChannel::IsSpeechMuted() const
{
	return TextToSpeech->IsMuted();
}

void FScreenReaderAnnouncementChannel::OnTextToSpeechFinishSpeaking()
{
	// reset the current announcement to null. This is necessary for the RequestSpeak() algorithm 
	CurrentAnnouncement = FScreenReaderAnnouncement();
	// if there are queued announcements, speak the next one 
	if (AnnouncementQueue.Num() > 0)
	{
		UE_LOGF(LogScreenReaderAnnouncement, VeryVerbose, "Announcing next queued announcement.");
		// @TODOAccessibility: We create a temp to ensure the RequestSpeak)() algorithm continues to work. Optimize in future 
		FScreenReaderAnnouncement NextAnnouncement;
		AnnouncementQueue.HeapPop(NextAnnouncement, FScreenReaderAnnouncement::FDefaultComparator());
		RequestSpeak(MoveTemp(NextAnnouncement));
	 }
}
