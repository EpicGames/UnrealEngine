// Copyright Epic Games, Inc. All Rights Reserved.

#include "Announcement/ScreenReaderWidgetAnnouncementBuilder.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"

#define LOCTEXT_NAMESPACE "ScreenReaderWidgetAnnouncementBuilkder"
const TMap<EAccessibleWidgetType, FText> FScreenReaderWidgetAnnouncementBuilder::WidgetTypeToAccessibleRoleMap
{ 
	{EAccessibleWidgetType::Button, LOCTEXT("ControlTypeButton", "button")},
	{EAccessibleWidgetType::CheckBox, LOCTEXT("ControlTypeCheckBox", "check box")},
	{EAccessibleWidgetType::ComboBox, LOCTEXT("ControlTypeComboBox", "combo box")},
	{EAccessibleWidgetType::Hyperlink, LOCTEXT("ControlTypeHyperlink", "Link")},
	{EAccessibleWidgetType::Image, LOCTEXT("ControlTypeImage", "image")},
	{EAccessibleWidgetType::Layout, LOCTEXT("ControlTypeLayout", "Layout")},
	{EAccessibleWidgetType::ScrollBar, LOCTEXT("ControlTypeScrollBar", "scroll bar")},
	{EAccessibleWidgetType::Slider, LOCTEXT("ControlTypeSlider", "slider")},
	{EAccessibleWidgetType::Text, LOCTEXT("ControlTypeText", "text")},
	{EAccessibleWidgetType::TextEdit, LOCTEXT("ControlTypeTextEdit", "edit")},
{EAccessibleWidgetType::Window, LOCTEXT("ControlTypeWindow", "window")},
{EAccessibleWidgetType::List, LOCTEXT("ControlTypeList", "List")},
	{EAccessibleWidgetType::ListItem, LOCTEXT("ControlTypeListItem", "ListItem")},
{EAccessibleWidgetType::ListItem, LOCTEXT("ControlTypeListItem", "ListItem")}
};

// @TODOAccessibility: These are just sample descriptions. Should go into settings or soemthing 
const TMap<EAccessibleWidgetType, FText> FScreenReaderWidgetAnnouncementBuilder::WidgetTypeToInteractionDescriptionMap
{
	{EAccessibleWidgetType::Button, LOCTEXT("InteractionTypeButton", "Press enter on the keyboard or the confirm button of the gamepad to activate.")},
	{EAccessibleWidgetType::CheckBox, LOCTEXT("InteractionTypeCheckBox", "Press enter on the keyboard or the confirm button of the gamepad to toggle.")},
	{EAccessibleWidgetType::ComboBox, LOCTEXT("InteractionTypeComboBox", "Press enter on the keyboard or the confirm button of the gamepad to focus. Repeat to expand.")},
	{EAccessibleWidgetType::Slider, LOCTEXT("InteractionTypeSlider", "Press enter on the keyboard or the confirm button of the gamepad to focus. Then press left and right to change the value.")},
	{EAccessibleWidgetType::Slider, LOCTEXT("InteractionTypeSlider", "Press enter on the keyboard or the confirm button of the gamepad to focus. Then press left and right to change the value.")},
	{EAccessibleWidgetType::List, LOCTEXT("InteractionTypeList", "Press up and down to navigate through the list items.")}
};

FString FScreenReaderWidgetAnnouncementBuilder::BuildWidgetAnnouncement(const TSharedRef<IAccessibleWidget>& InWidget)
{
	// format is label, value, role. Help text. Interaction desription.
	// This follows formatting of standard OS screen readers 
	TStringBuilder<256> Announcement;
	Announcement.Append(BuildLabel(InWidget));
	Announcement.Append(TEXT(", "));
	Announcement.Append(BuildValue(InWidget));
	Announcement.Append(TEXT(", "));	
	Announcement.Append(BuildRole(InWidget));
	Announcement.Append(TEXT(". "));
	Announcement.Append(BuildHelpText(InWidget));
	Announcement.Append(TEXT(". "));
	Announcement.Append(BuildInteractionDescription(InWidget));
	Announcement.Append(TEXT("."));

	return Announcement.ToString();
}

FString FScreenReaderWidgetAnnouncementBuilder::BuildLabel(const TSharedRef<IAccessibleWidget>& InWidget) const
{
	return InWidget->GetWidgetName();
}

FString FScreenReaderWidgetAnnouncementBuilder::BuildRole(const TSharedRef<IAccessibleWidget>& InWidget) const
{
	static const FText UnknownAccessibleRole = LOCTEXT("ControlTypeUnknown", "Unknown");
	const FText* LocalizedAccessibleRole = FScreenReaderWidgetAnnouncementBuilder::WidgetTypeToAccessibleRoleMap.Find(InWidget->GetWidgetType());
	return LocalizedAccessibleRole ? LocalizedAccessibleRole->ToString() : UnknownAccessibleRole .ToString();
}

FString FScreenReaderWidgetAnnouncementBuilder::BuildValue(const TSharedRef<IAccessibleWidget>& InWidget) const
{
	if (!InWidget->AsProperty())
	{
		return FString();
	}
	switch (InWidget->GetWidgetType())
	{
	case EAccessibleWidgetType::CheckBox:
	{
		if (InWidget->AsActivatable()->GetCheckedState())
		{
			static const FText CheckedState= LOCTEXT("CheckBoxChecked", "Checked");
			return CheckedState.ToString();
		}
		else
		{
			static const FText UncheckedState = LOCTEXT("CheckBoxUnchecked", "Unchecked");
			return UncheckedState.ToString();
		}
	}
	case EAccessibleWidgetType::TextEdit:
	{
		// only return a value if the field is not a password
		if (!InWidget->AsProperty()->IsPassword())
		{
			return InWidget->AsProperty()->GetValue();
		}
		else
		{
			static const FText PasswordHidden= LOCTEXT("TextFieldPasswordHidden", "Hidden");
			return PasswordHidden.ToString();
		}
	}
	default:
		return InWidget->AsProperty()->GetValue();
	}
}

FString FScreenReaderWidgetAnnouncementBuilder::BuildHelpText(const TSharedRef<IAccessibleWidget>& InWidget) const
{
	return InWidget->GetHelpText();
}
FString FScreenReaderWidgetAnnouncementBuilder::BuildInteractionDescription(const TSharedRef<IAccessibleWidget>& InWidget) const
{
	const FText* LocalizedInteractionDescription = FScreenReaderWidgetAnnouncementBuilder::WidgetTypeToInteractionDescriptionMap.Find(InWidget->GetWidgetType());
	return LocalizedInteractionDescription ? LocalizedInteractionDescription->ToString() : FString();
}


#undef LOCTEXT_NAMESPACE
