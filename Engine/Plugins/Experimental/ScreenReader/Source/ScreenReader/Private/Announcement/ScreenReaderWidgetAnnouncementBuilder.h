// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/SharedPointer.h"

class IAccessibleWidget;
enum class EAccessibleWidgetType : uint8;


class FScreenReaderWidgetAnnouncementBuilder
{
public:
	FScreenReaderWidgetAnnouncementBuilder() = default;
	~FScreenReaderWidgetAnnouncementBuilder() = default;
	FString BuildWidgetAnnouncement(const TSharedRef<IAccessibleWidget>& InWidget);

private:
	FString BuildLabel(const TSharedRef<IAccessibleWidget>& InWidget) const;
	FString BuildRole(const TSharedRef<IAccessibleWidget>& InWidget) const;
	FString BuildValue(const TSharedRef<IAccessibleWidget>& InWidget) const;
	FString BuildHelpText(const TSharedRef<IAccessibleWidget>& InWidget) const;
	FString BuildInteractionDescription(const TSharedRef<IAccessibleWidget>& InWidget) const;

	static const TMap<EAccessibleWidgetType, FText> WidgetTypeToAccessibleRoleMap;
	static const TMap<EAccessibleWidgetType, FText> WidgetTypeToInteractionDescriptionMap;
};

