// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/ScreenReaderApplicationMessageHandlerBase.h"

FScreenReaderApplicationMessageHandlerBase::FScreenReaderApplicationMessageHandlerBase(const TSharedRef<FGenericApplicationMessageHandler>& InTargetMessageHandler, FScreenReaderBase& InOwningScreenReader)
	: OwningScreenReader(InOwningScreenReader)
	, TargetMessageHandler(InTargetMessageHandler)
	, bProcessMouseInput(false)
	, bProcessKeyInput(true)
{

}

// FGenericApplicationMessageHandler
bool FScreenReaderApplicationMessageHandlerBase::ShouldProcessUserInputMessages(const TSharedPtr< FGenericWindow >& PlatformWindow) const
{
	return TargetMessageHandler->ShouldProcessUserInputMessages(PlatformWindow);
}

bool FScreenReaderApplicationMessageHandlerBase::OnKeyChar(const TCHAR Character, const bool IsRepeat)
{
	return TargetMessageHandler->OnKeyChar(Character, IsRepeat);
}

bool FScreenReaderApplicationMessageHandlerBase::OnKeyDown(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat)
{
	return TargetMessageHandler->OnKeyDown(KeyCode, CharacterCode, IsRepeat);
}

bool FScreenReaderApplicationMessageHandlerBase::OnKeyUp(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat)
{
	return TargetMessageHandler->OnKeyUp(KeyCode, CharacterCode, IsRepeat);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button)
{
	return TargetMessageHandler->OnMouseDown(Window, Button);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos)
{
	return TargetMessageHandler->OnMouseDown(Window, Button, CursorPos);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMouseUp(const EMouseButtons::Type Button)
{
	return TargetMessageHandler->OnMouseUp(Button);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMouseUp(const EMouseButtons::Type Button, const FVector2D CursorPos)
{
	return TargetMessageHandler->OnMouseUp(Button, CursorPos);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button)
{
	return TargetMessageHandler->OnMouseDoubleClick(Window, Button);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos)
{
	return TargetMessageHandler->OnMouseDoubleClick(Window, Button, CursorPos);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMouseWheel(const float Delta)
{
	return TargetMessageHandler->OnMouseWheel(Delta);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMouseWheel(const float Delta, const FVector2D CursorPos)
{
	return TargetMessageHandler->OnMouseWheel(Delta, CursorPos);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMouseMove()
{
	return TargetMessageHandler->OnMouseMove();
}

bool FScreenReaderApplicationMessageHandlerBase::OnRawMouseMove(const int32 X, const int32 Y)
{
	return TargetMessageHandler->OnRawMouseMove(X, Y);
}

bool FScreenReaderApplicationMessageHandlerBase::OnCursorSet()
{
	return TargetMessageHandler->OnCursorSet();
}

bool FScreenReaderApplicationMessageHandlerBase::OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue)
{
	return TargetMessageHandler->OnControllerAnalog(KeyName, PlatformUserId, InputDeviceId, AnalogValue);
}

bool FScreenReaderApplicationMessageHandlerBase::OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	return TargetMessageHandler->OnControllerButtonPressed(KeyName, PlatformUserId, InputDeviceId, IsRepeat);
}

bool FScreenReaderApplicationMessageHandlerBase::OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	return TargetMessageHandler->OnControllerButtonReleased(KeyName, PlatformUserId, InputDeviceId, IsRepeat);
}

void FScreenReaderApplicationMessageHandlerBase::OnBeginGesture()
{
	TargetMessageHandler->OnBeginGesture();
}

bool FScreenReaderApplicationMessageHandlerBase::OnTouchGesture(EGestureEvent GestureType, const FVector2D& Delta, float WheelDelta, bool bIsDirectionInvertedFromDevice)
{
	return TargetMessageHandler->OnTouchGesture(GestureType, Delta, WheelDelta, bIsDirectionInvertedFromDevice);
}

void FScreenReaderApplicationMessageHandlerBase::OnEndGesture()
{
	TargetMessageHandler->OnEndGesture();
}

bool FScreenReaderApplicationMessageHandlerBase::OnTouchStarted(const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	return TargetMessageHandler->OnTouchStarted(Window, Location, Force, TouchIndex, ControllerId);
}

bool FScreenReaderApplicationMessageHandlerBase::OnTouchMoved(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	return TargetMessageHandler->OnTouchMoved(Location, Force, TouchIndex, ControllerId);
}

bool FScreenReaderApplicationMessageHandlerBase::OnTouchEnded(const FVector2D& Location, int32 TouchIndex, int32 ControllerId)
{
	return TargetMessageHandler->OnTouchEnded(Location, TouchIndex, ControllerId);
}

bool FScreenReaderApplicationMessageHandlerBase::OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	return TargetMessageHandler->OnTouchForceChanged(Location, Force, TouchIndex, ControllerId);
}

bool FScreenReaderApplicationMessageHandlerBase::OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId)
{
	return TargetMessageHandler->OnTouchFirstMove(Location, Force, TouchIndex, ControllerId);
}

void FScreenReaderApplicationMessageHandlerBase::ShouldSimulateGesture(EGestureEvent Gesture, bool bEnable)
{
	return TargetMessageHandler->ShouldSimulateGesture(Gesture, bEnable);
}

bool FScreenReaderApplicationMessageHandlerBase::OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, int32 ControllerId)
{
	return TargetMessageHandler->OnMotionDetected(Tilt, RotationRate, Gravity, Acceleration, ControllerId);
}

bool FScreenReaderApplicationMessageHandlerBase::OnSizeChanged(const TSharedRef< FGenericWindow >& Window, const int32 Width, const int32 Height, bool bWasMinimized /* = false */)
{
	return TargetMessageHandler->OnSizeChanged(Window, Width, Height, bWasMinimized);
}

void FScreenReaderApplicationMessageHandlerBase::OnOSPaint(const TSharedRef<FGenericWindow>& Window)
{
	return TargetMessageHandler->OnOSPaint(Window);
}

FWindowSizeLimits FScreenReaderApplicationMessageHandlerBase::GetSizeLimitsForWindow(const TSharedRef<FGenericWindow>& Window) const
{
	return TargetMessageHandler->GetSizeLimitsForWindow(Window);
}

void FScreenReaderApplicationMessageHandlerBase::OnResizingWindow(const TSharedRef< FGenericWindow >& Window)
{
	TargetMessageHandler->OnResizingWindow(Window);
}

bool FScreenReaderApplicationMessageHandlerBase::BeginReshapingWindow(const TSharedRef< FGenericWindow >& Window)
{
	return TargetMessageHandler->BeginReshapingWindow(Window);
}

void FScreenReaderApplicationMessageHandlerBase::FinishedReshapingWindow(const TSharedRef< FGenericWindow >& Window)
{
	return TargetMessageHandler->FinishedReshapingWindow(Window);
}

void FScreenReaderApplicationMessageHandlerBase::HandleDPIScaleChanged(const TSharedRef< FGenericWindow >& Window)
{
	TargetMessageHandler->HandleDPIScaleChanged(Window);
}

void FScreenReaderApplicationMessageHandlerBase::OnMovedWindow(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y)
{
	TargetMessageHandler->OnMovedWindow(Window, X, Y);
}

bool FScreenReaderApplicationMessageHandlerBase::OnWindowActivationChanged(const TSharedRef< FGenericWindow >& Window, const EWindowActivation ActivationType)
{
	return TargetMessageHandler->OnWindowActivationChanged(Window, ActivationType);
}

bool FScreenReaderApplicationMessageHandlerBase::OnApplicationActivationChanged(const bool IsActive)
{
	return TargetMessageHandler->OnApplicationActivationChanged(IsActive);
}

bool FScreenReaderApplicationMessageHandlerBase::OnConvertibleLaptopModeChanged()
{
	return TargetMessageHandler->OnConvertibleLaptopModeChanged();
}

EWindowZone::Type FScreenReaderApplicationMessageHandlerBase::GetWindowZoneForPoint(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y)
{
	return TargetMessageHandler->GetWindowZoneForPoint(Window, X, Y);
}

void FScreenReaderApplicationMessageHandlerBase::OnWindowClose(const TSharedRef< FGenericWindow >& Window)
{
	TargetMessageHandler->OnWindowClose(Window);
}

EDropEffect::Type FScreenReaderApplicationMessageHandlerBase::OnDragEnterText(const TSharedRef< FGenericWindow >& Window, const FString& Text)
{
	return TargetMessageHandler->OnDragEnterText(Window, Text);
}

EDropEffect::Type FScreenReaderApplicationMessageHandlerBase::OnDragEnterFiles(const TSharedRef< FGenericWindow >& Window, const TArray< FString >& Files)
{
	return TargetMessageHandler->OnDragEnterFiles(Window, Files);
}

EDropEffect::Type FScreenReaderApplicationMessageHandlerBase::OnDragEnterExternal(const TSharedRef< FGenericWindow >& Window, const FString& Text, const TArray< FString >& Files)
{
	return TargetMessageHandler->OnDragEnterExternal(Window, Text, Files);
}

EDropEffect::Type FScreenReaderApplicationMessageHandlerBase::OnDragOver(const TSharedPtr< FGenericWindow >& Window)
{
	return TargetMessageHandler->OnDragOver(Window);
}

void FScreenReaderApplicationMessageHandlerBase::OnDragLeave(const TSharedPtr< FGenericWindow >& Window)
{
	TargetMessageHandler->OnDragLeave(Window);
}

EDropEffect::Type FScreenReaderApplicationMessageHandlerBase::OnDragDrop(const TSharedPtr< FGenericWindow >& Window)
{
	return TargetMessageHandler->OnDragDrop(Window);
}

bool FScreenReaderApplicationMessageHandlerBase::OnWindowAction(const TSharedRef< FGenericWindow >& Window, const EWindowAction::Type InActionType)
{
	return TargetMessageHandler->OnWindowAction(Window, InActionType);
}
// ~
