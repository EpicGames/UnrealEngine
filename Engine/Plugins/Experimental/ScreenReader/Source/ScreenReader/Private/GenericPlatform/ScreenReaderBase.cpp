// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/ScreenReaderBase.h"
#include "GenericPlatform/ScreenReaderApplicationMessageHandlerBase.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/ScreenReaderUser.h"
#include "ScreenReaderLog.h"


FScreenReaderBase::FScreenReaderBase(const TSharedRef<GenericApplication>& InPlatformApplication)
	: ScreenReaderApplicationMessageHandler(MakeShared<FScreenReaderApplicationMessageHandlerBase>(InPlatformApplication->GetMessageHandler(), *this))
	, PlatformApplication(InPlatformApplication)
	, bActive(false)
{

}

FScreenReaderBase::~FScreenReaderBase()
{

}

void FScreenReaderBase::Activate()
{
	if (!IsActive())
	{
		checkf(PlatformApplication.IsValid(), TEXT("Trying to activate the screen reader with an invalid platform aplication."));
		UE_LOGF(LogScreenReader, Verbose, "Activating screen reader.");
		// Set the screen reader application message handler to be the platform's message handler 
		// This allows the screen reader to intercept all applications before passing them on to Slate 
		PlatformApplication.Pin()->SetMessageHandler(ScreenReaderApplicationMessageHandler);
		TSharedRef<FGenericAccessibleMessageHandler> AccessibleMessageHandler = PlatformApplication.Pin()->GetAccessibleMessageHandler();
		// We unbind the delegate just in case it is still bound to some other target. This ensures that all accessible events now only get processed by the screen reader via OnAccessibleEventRaised
		AccessibleMessageHandler->UnbindAccessibleEventDelegate();
		AccessibleMessageHandler->SetAccessibleEventDelegate(FGenericAccessibleMessageHandler::FAccessibleEvent::CreateRaw(this, &FScreenReaderBase::OnAccessibleEventRaised));
		OnActivate();
		bActive = true;
}
}

void FScreenReaderBase::Deactivate()
{
	if (IsActive())
	{
		UE_LOGF(LogScreenReader, Verbose, "Deactivating screen reader.");
		OnDeactivate();
		TSharedRef<FGenericAccessibleMessageHandler> AccessibleMessageHandler = PlatformApplication.Pin()->GetAccessibleMessageHandler();
		AccessibleMessageHandler->UnbindAccessibleEventDelegate();
		// Set FSlateApplication as the platform message ahndler again 
		PlatformApplication.Pin()->SetMessageHandler(ScreenReaderApplicationMessageHandler->GetTargetMessageHandler());
		bActive = false;
	}
}

bool FScreenReaderBase::RegisterUser(int32 InUserId)
{
	FGenericAccessibleUserRegistry& UserRegistry = PlatformApplication.Pin()->GetAccessibleMessageHandler()->GetAccessibleUserRegistry();
	// @TODOAccessibility: Create a mechanism for creating the appropriate user. Factory or abstract factory something like that
	TSharedRef<FScreenReaderUser> ScreenReaderUser = MakeShared<FScreenReaderUser>(InUserId);
	return UserRegistry.RegisterUser(ScreenReaderUser);
}

bool FScreenReaderBase::UnregisterUser(int32 InUserId)
{
	FGenericAccessibleUserRegistry& UserRegistry = PlatformApplication.Pin()->GetAccessibleMessageHandler()->GetAccessibleUserRegistry();
	return UserRegistry.UnregisterUser(InUserId);
}

bool FScreenReaderBase::IsUserRegistered(int32 InUserId) const
{
	const FGenericAccessibleUserRegistry& UserRegistry = PlatformApplication.Pin()->GetAccessibleMessageHandler()->GetAccessibleUserRegistry();
	return UserRegistry.IsUserRegistered(InUserId);
}

void FScreenReaderBase::UnregisterAllUsers()
{
	FGenericAccessibleUserRegistry& UserRegistry = PlatformApplication.Pin()->GetAccessibleMessageHandler()->GetAccessibleUserRegistry();
	UserRegistry.UnregisterAllUsers();
}

TSharedRef<FScreenReaderUser> FScreenReaderBase::GetUserChecked(int32 InUserId) const
{
	const FGenericAccessibleUserRegistry& UserRegistry = PlatformApplication.Pin()->GetAccessibleMessageHandler()->GetAccessibleUserRegistry();
	checkf(UserRegistry.IsUserRegistered(InUserId), TEXT("User Id %d is not registered. Did you forget to register the user with RegisterUser()?"), InUserId);
	return StaticCastSharedRef<FScreenReaderUser>(UserRegistry.GetUser(InUserId).ToSharedRef());
}
	
TSharedPtr<FScreenReaderUser> FScreenReaderBase::GetUser(int32 InUserId) const
{
	const FGenericAccessibleUserRegistry& UserRegistry = PlatformApplication.Pin()->GetAccessibleMessageHandler()->GetAccessibleUserRegistry();
	return StaticCastSharedPtr<FScreenReaderUser>(UserRegistry.GetUser(InUserId));
}

