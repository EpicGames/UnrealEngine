// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatform/ScreenReaderUser.h"
#include "Announcement/ScreenReaderAnnouncementChannel.h"
#include "GenericPlatform/ITextToSpeechFactory.h"
#include "Navigation/ScreenReaderNavigationPolicy.h"
#include "GenericPlatform/ScreenReaderReply.h"
#include "ScreenReaderLog.h"
#include "TextToSpeechModule.h"

FScreenReaderUser::FScreenReaderUser(FAccessibleUserIndex InUserIndex)
	: FGenericAccessibleUser(InUserIndex)
	, NavigationPolicy(MakeShared<FScreenReaderDefaultNavigationPolicy>())
	, bActive(false)
{
	// @TODOAccessibility: For now, just default it to the platform TTS. Should give a way to allow users to use custom TTS though 
	AnnouncementChannel = MakeUnique<FScreenReaderAnnouncementChannel>(ITextToSpeechModule::Get().GetPlatformFactory()->Create());
}

FScreenReaderUser::~FScreenReaderUser()
{
	Deactivate();
}

void FScreenReaderUser::Activate()
{
	if (!IsActive())
	{
		bActive = true;
		AnnouncementChannel->Activate();
	}
}

void FScreenReaderUser::Deactivate()
{
	if (IsActive())
	{
		bActive = false;
		AnnouncementChannel->Deactivate();
	}
}

FScreenReaderReply FScreenReaderUser::RequestSpeak(FScreenReaderAnnouncement InAnnouncement)
{
	if (IsActive())
	{
		return AnnouncementChannel->RequestSpeak(MoveTemp(InAnnouncement));
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply FScreenReaderUser::StopSpeaking()
{
	if (IsActive())
	{
		return AnnouncementChannel->StopSpeaking();
	}
	return FScreenReaderReply::Unhandled();
}

bool FScreenReaderUser::IsSpeaking() const
{
	if (IsActive())
	{
		return AnnouncementChannel->IsSpeaking();
	}
	return false;
}

FScreenReaderReply FScreenReaderUser::RequestSpeakWidget(const TSharedRef<IAccessibleWidget>& InWidget)
{
	if (IsActive())
	{
		return AnnouncementChannel->RequestSpeakWidget(InWidget);
	}
	return FScreenReaderReply::Unhandled();
}

float FScreenReaderUser::GetSpeechVolume() const
{
	return AnnouncementChannel->GetSpeechVolume();
}

FScreenReaderReply FScreenReaderUser::SetSpeechVolume(float InVolume)
{
	return AnnouncementChannel->SetSpeechVolume(InVolume);
}

float FScreenReaderUser::GetSpeechRate() const
{
	return AnnouncementChannel->GetSpeechRate();
}

FScreenReaderReply FScreenReaderUser::SetSpeechRate(float InRate)
{
	return AnnouncementChannel->SetSpeechRate(InRate);
}

FScreenReaderReply FScreenReaderUser::MuteSpeech()
{
	if (IsActive())
	{
		return AnnouncementChannel->MuteSpeech();
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply FScreenReaderUser::UnmuteSpeech()
{
	if (IsActive())
	{
		return AnnouncementChannel->UnmuteSpeech();
	}

	return FScreenReaderReply::Unhandled();
}

bool FScreenReaderUser::IsSpeechMuted() const
{
	return AnnouncementChannel->IsSpeechMuted();
}

void FScreenReaderUser::OnUnregistered()
{
	Deactivate();
}

void FScreenReaderUser::SetNavigationPolicy(const TSharedRef<IScreenReaderNavigationPolicy>& InNavigationPolicy)
{
	NavigationPolicy = InNavigationPolicy;
}

FScreenReaderReply FScreenReaderUser::NavigateToNextSibling()
{
	TSharedPtr<IAccessibleWidget> CurrentFocusedAccessibleWidget = GetFocusedAccessibleWidget();
	if (CurrentFocusedAccessibleWidget)
	{
		TSharedPtr<IAccessibleWidget> NextSibling = NavigationPolicy->GetNextSiblingFrom(CurrentFocusedAccessibleWidget.ToSharedRef());
		if (NextSibling && NextSibling->SetUserFocus(GetIndex()))
		{
			UE_LOGF(LogScreenReaderNavigation, Verbose, "User %d now focused on next sibling %ls.", GetIndex(), *NextSibling->ToString());
			return FScreenReaderReply::Handled();
		}
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply FScreenReaderUser::NavigateToPreviousSibling()
{
	TSharedPtr<IAccessibleWidget> CurrentFocusedAccessibleWidget = GetFocusedAccessibleWidget();
	if (CurrentFocusedAccessibleWidget)
	{
		TSharedPtr<IAccessibleWidget> PreviousSibling = NavigationPolicy->GetPreviousSiblingFrom(CurrentFocusedAccessibleWidget.ToSharedRef());
		if (PreviousSibling && PreviousSibling->SetUserFocus(GetIndex()))
		{
			UE_LOGF(LogScreenReaderNavigation, Verbose, "User %d now focused on previous sibling %ls.", GetIndex(), *PreviousSibling->ToString());
			return FScreenReaderReply::Handled();
		}
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply FScreenReaderUser::NavigateToFirstAncestor()
{
	TSharedPtr<IAccessibleWidget> CurrentFocusedAccessibleWidget = GetFocusedAccessibleWidget();
	if (CurrentFocusedAccessibleWidget)
	{
		TSharedPtr<IAccessibleWidget> Ancestor = NavigationPolicy->GetFirstAncestorFrom(CurrentFocusedAccessibleWidget.ToSharedRef());
		if (Ancestor && Ancestor->SetUserFocus(GetIndex()))
		{
			UE_LOGF(LogScreenReaderNavigation, Verbose, "User %d now focused on ancestor %ls.", GetIndex(), *Ancestor->ToString());
			return FScreenReaderReply::Handled();
		}
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply FScreenReaderUser::NavigateToFirstChild()
{
	TSharedPtr<IAccessibleWidget> CurrentFocusedAccessibleWidget = GetFocusedAccessibleWidget();
	if (CurrentFocusedAccessibleWidget)
	{
		TSharedPtr<IAccessibleWidget> Child = NavigationPolicy->GetFirstChildFrom(CurrentFocusedAccessibleWidget.ToSharedRef());
		if (Child && Child->SetUserFocus(GetIndex()))
		{
			UE_LOGF(LogScreenReaderNavigation, Verbose, "User %d now focused on first child %ls.", GetIndex(), *Child->ToString());
			return FScreenReaderReply::Handled();
		}
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply FScreenReaderUser::NavigateToNextWidgetInHierarchy()
{
	TSharedPtr<IAccessibleWidget> CurrentFocusedAccessibleWidget = GetFocusedAccessibleWidget();
	if (CurrentFocusedAccessibleWidget)
	{
		TSharedPtr<IAccessibleWidget> NextWidget = NavigationPolicy->GetNextWidgetInHierarchyFrom(CurrentFocusedAccessibleWidget.ToSharedRef());
		if (NextWidget && NextWidget->SetUserFocus(GetIndex()))
		{
			UE_LOGF(LogScreenReaderNavigation, Verbose, "User %d now focused on next widget %ls.", GetIndex(), *NextWidget->ToString());
			return FScreenReaderReply::Handled();
		}
	}
	return FScreenReaderReply::Unhandled();
}

FScreenReaderReply FScreenReaderUser::NavigateToPreviousWidgetInHierarchy()
{
	TSharedPtr<IAccessibleWidget> CurrentFocusedAccessibleWidget = GetFocusedAccessibleWidget();
	if (CurrentFocusedAccessibleWidget)
	{
		TSharedPtr<IAccessibleWidget> PreviousWidget = NavigationPolicy->GetPreviousWidgetInHierarchyFrom(CurrentFocusedAccessibleWidget.ToSharedRef());
		if (PreviousWidget && PreviousWidget->SetUserFocus(GetIndex()))
		{
			UE_LOGF(LogScreenReaderNavigation, Verbose, "User %d now focused on previous widget %ls.", GetIndex(), *PreviousWidget->ToString());
			return FScreenReaderReply::Handled();
		}
	}
	return FScreenReaderReply::Unhandled();
}

