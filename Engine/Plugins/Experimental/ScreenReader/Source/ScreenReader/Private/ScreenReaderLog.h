// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogScreenReader, Log, All);

DECLARE_LOG_CATEGORY_EXTERN(LogScreenReaderInput, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogScreenReaderAnnouncement, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogScreenReaderNavigation, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogScreenReaderAction, Log, All);

