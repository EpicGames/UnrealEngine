// Copyright Epic Games, Inc. All Rights Reserved.


#include "ScreenReaderLog.h"
#include "Modules/ModuleManager.h"

static_assert(WITH_ACCESSIBILITY, "Trying to use the screen reader plugin with accessibility disabled. Accessibility must be enabled to use this plugin. Either enable accessibility or disable the plugin.");

IMPLEMENT_MODULE(FDefaultModuleImpl, ScreenReader);

DEFINE_LOG_CATEGORY(LogScreenReader);
DEFINE_LOG_CATEGORY(LogScreenReaderInput);
DEFINE_LOG_CATEGORY(LogScreenReaderAnnouncement);
DEFINE_LOG_CATEGORY(LogScreenReaderNavigation);
DEFINE_LOG_CATEGORY(LogScreenReaderAction);

