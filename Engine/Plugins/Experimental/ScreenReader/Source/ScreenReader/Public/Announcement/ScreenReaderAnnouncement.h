// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "ScreenReaderAnnouncement.generated.h"

/**
* The priority level associated with a screen reader announcement.
* The priority of an announcement could be used to determine if an announcement A could interrupt announcement B
* or to help order the announcements if multiple are queued.
*/
UENUM()
enum class EScreenReaderAnnouncementPriority : uint8
{
	High = 0,
	Medium,
	Low
};

/**
* A struct that contains information about how a screen reader announcement should behave when the announcement is requested to be spoken.
* This struct holds information to determine an announcement's priority in relation to other announcements and determines if an 
* announcement can be interrupted by other announcements and if an announcement should be queued if it cannot be made immendiately.
* For C++ users, please use the static methods provided to create instances of FScreenReaderInfo with pre-defined defaults that will
* satisfy 80% of your use cases.
* Please see FScreenReaderUser::RequestSpeak() for examples of how to use this struct.
* @see FScreenReaderUser, FScreenReaderAnnouncement
*/
USTRUCT(BlueprintType)
struct SCREENREADER_API FScreenReaderAnnouncementInfo
{
	GENERATED_BODY()
public:
	// Only used to satisfy USTRUCT constraints 
	// Please use ctor that takes parameters 
	FScreenReaderAnnouncementInfo() = default;
	FScreenReaderAnnouncementInfo(bool bInShouldQueue, bool bInInterruptable, EScreenReaderAnnouncementPriority InPriority);
	bool ShouldQueue() const 
	{ 
		return bShouldQueue; 
	}
	bool IsInterruptable() const 
	{ 
		return bInterruptable; 
	}
	EScreenReaderAnnouncementPriority GetPriority() const 
	{ 
		return Priority; 
	}
	double GetTimestamp() const 
	{ 
		return Timestamp; 
	}
	/** Returns a string that represents all of the data reflected in this struct. */
	FString ToString() const;

	/** 
	* The announcement info used for announcements concerning widgets and their content.
	* These are the lowest priority, are interruptable and will not be queued if interrupted.
	* Use this when requestin any announcements related to widgets that the user can easily access again
	* E.g Useful for announcing widgets that the user is currently focused on. The user can always refocus on the widget to hear its contents spoken so
	* it doesn't matter if it is interrupted.
	*/
	static FScreenReaderAnnouncementInfo DefaultWidgetAnnouncement()
	{
		return FScreenReaderAnnouncementInfo(false, true, EScreenReaderAnnouncementPriority::Low);
	}
	/**
	* An announcement info used for announcements that are of critical importance to the end user.
	* Announcements with this announcement info have the highest priority, are not interruptable by other announcements and will be queued if multiple important announcements are requested.
	* Use this for communicating critical information to an end user.
	* E.g Use this for announcements concerning the system (battery low, accessibility enabled etc) or important in game events that cannot be revisted or played again.
	*/
	static FScreenReaderAnnouncementInfo Important()
	{
		return FScreenReaderAnnouncementInfo(true, false, EScreenReaderAnnouncementPriority::High);
	}

	/**
	* This announcement info is used for announcements to provide feedback for end user actions or inform them of changes.
	* Announcements with this announcement info have medium priority, can be interrupted and will not be queued if interrupted by an announcement of equal or higher priority.
	* User this announcement info for announcements that provide feedback to the end user.
	* E.g When they have successfully activated a UI element, the screen has changed to display a different meny or when new elements have appeared on screen that the user should know about.
	*/
	FScreenReaderAnnouncementInfo UserFeedback()
	{
		return FScreenReaderAnnouncementInfo(false, true, EScreenReaderAnnouncementPriority::Medium);
	}

private:
	/** True if the associated announcement should be queued if it cannot be spoken immediately or is interrupted. Else false.*/
	UPROPERTY(BlueprintReadWrite, Category="ScreenReaderAnnouncementInfo", meta = (AllowPrivateAccess = "true"))
	bool bShouldQueue = false;
	/** True if the associazted announcement can be intrrupted by another announcement. Else false. */
	UPROPERTY(BlueprintReadWrite, Category="ScreenReaderAnnouncementInfo", meta = (AllowPrivateAccess = "true"))
	bool bInterruptable= true;
	/** The priority level of the associated announcement. */
	UPROPERTY(BlueprintReadWrite, Category="ScreenReaderAnnouncementInfo", meta = (AllowPrivateAccess = "true"))
	EScreenReaderAnnouncementPriority Priority = EScreenReaderAnnouncementPriority::Medium;
	// @TODOAccessibility: Update timestamp to be recorded when announcements are requested, not when they are constructed. 
	/** The timestamp for when the associated announcement was created. */
	double Timestamp = FPlatformTime::Seconds();
};

/**
* A screen reader announcement represents a localized message that a user wants to speak to an end user.
* This class is used to request announcements to be spoken to an end user via the FScreenReaderUser class.
* FScreenReaderUser::RequestSpeak() for instructions on how to use the class.
* @see FScreenReaderUser
*/
USTRUCT(BlueprintType)
struct SCREENREADER_API FScreenReaderAnnouncement
{
GENERATED_BODY()
public:
	// Only used to satisfy USTRUCT requirements
	// Please use ctor that takes arguments
	FScreenReaderAnnouncement() = default;
	/**
	* The constructor that should be used in most cases.
	* @param InAnnouncementString The localized string that a user wishes to rquest to speak to a user
	FScreenReaderAnnouncement(FString InAnnouncementString, FScreenReaderAnnouncementInfo );
	* @param The announcement info associated with the announcement string to control if this announcement can be interrupted or queued
	* when a request the speak this announcement is made.
	*/
	FScreenReaderAnnouncement(FString InAnnouncementString, FScreenReaderAnnouncementInfo );
	const FString& GetAnnouncementString() const { return AnnouncementString; }
	const FScreenReaderAnnouncementInfo& GetAnnouncementInfo() const { return AnnouncementInfo; }
	/** Converts the contents of the announcement to a string for easy printing. Useful for debugging. */
	FString ToString() const;

	/** 
	* The default comparator for screen reader announcements. Can be used to order screen announcements in a priority queue
	* Users can provide their own comaprators by implementing the same interface.
	*/
	struct FDefaultComparator
	{
		/** Returns true if A is less than B */
		bool operator()(const FScreenReaderAnnouncement& A, const FScreenReaderAnnouncement& B) const
		{
			// primary key is priority 
			// secondary key is time 
			return A.GetAnnouncementInfo().GetPriority() == B.GetAnnouncementInfo().GetPriority()
				? A.GetAnnouncementInfo().GetTimestamp() < B.GetAnnouncementInfo().GetTimestamp()
				: A.GetAnnouncementInfo().GetPriority() < B.GetAnnouncementInfo().GetPriority();
		}
	};

	/**
	* The default interruption policy to determine if announcement A can be interrupted by announcement B.
	* Users can provide their own implementation by creating their own struct and implementing the same interface.
	*/
	struct FDefaultInterruptionPolicy
	{
		/** Returns true if the interruptee can be interrupted by the interruptor. Else returns false */
		bool CanBeInterruptedBy(const FScreenReaderAnnouncement& Interruptee, const FScreenReaderAnnouncement& Interrupter) const
		{
			// priority comparison is >= because High Priority is 0 and Low Priority is 2
			return Interruptee.GetAnnouncementInfo().IsInterruptable()
				? Interruptee.GetAnnouncementInfo().GetPriority() >= Interrupter.GetAnnouncementInfo().GetPriority()
				: false;
		}
	};

	/**
	* A static function that determines if one announcement can be interrupted by another based on a provided interruption policy.true
	* @param Interruptee The announcement that the Interruptor will try to interrupt
	* @param Interrupter The announcement that is trying to interrupt Interruptee
	* @param InInterruptionPolicy The policy used to determine if the Interruptee can be interrupted by the Interrupter. Users can create their own 
	* interruption policies that satisfy the interface provided by FDefaultInterruptionPolicy.
	* @return Returns true if the Interruptee can be interrupted by the Interrupter based on the passed in interruption policy. Else returns false.
	* @see FDefaultInterruptionPolicy
	*/
	template<typename InterruptionPolicy>
	static bool CanBeInterruptedBy(const FScreenReaderAnnouncement& Interruptee, const FScreenReaderAnnouncement& Interrupter, const InterruptionPolicy& InInterruptionPolicy)
	{
		return InInterruptionPolicy.CanBeInterruptedBy(Interruptee, Interrupter);
	}
public:
	/** A localized string that represents the message to be spoken to a end user */
	UPROPERTY(BlueprintReadWrite, Category="ScreenReaderAnnouncement")
	FString AnnouncementString;
	/** The announcement info associated with the announcement which controls how the announcement behaves when a user requests this announcement to be spoken to an end user */
	UPROPERTY(BlueprintReadWrite, Category="ScreenReaderAnnouncement")
	FScreenReaderAnnouncementInfo AnnouncementInfo;
};
