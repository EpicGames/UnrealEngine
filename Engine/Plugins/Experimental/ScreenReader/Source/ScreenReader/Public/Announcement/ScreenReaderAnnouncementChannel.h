// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Announcement/ScreenReaderAnnouncement.h"
#include "GenericPlatform/ScreenReaderReply.h"


class FTextToSpeechBase;
class IAccessibleWidget;

/**
* An announcement channel handles processing of announcements and speaking announcements through text to speech (TTS).
* Announcements are forwarded here from a screen reader user and the announcements will be spoken immediately, queued or interrupt the currently spoken announcement.
* See FScreenReaderUser for an explanation of these functions and FScreenReaderAnnouncement for how announcements are used.
* @see FScreenReaderUser, FScreenReaderAnnouncement, FScreenReaderAnnouncementInfo
*/
class SCREENREADER_API FScreenReaderAnnouncementChannel
{
public:
	explicit FScreenReaderAnnouncementChannel(const TSharedRef<FTextToSpeechBase>& InTextToSpeech);
	virtual ~FScreenReaderAnnouncementChannel();
	FScreenReaderAnnouncementChannel(const FScreenReaderAnnouncementChannel& OtherAnnouncementChannel) = delete;
	FScreenReaderAnnouncementChannel& operator=(const FScreenReaderAnnouncementChannel& OtherAnnouncementChannel) = delete;

	void Activate();
	void Deactivate();

	FScreenReaderReply RequestSpeak(FScreenReaderAnnouncement InAnnouncement);
	FScreenReaderReply StopSpeaking();
	bool IsSpeaking() const;
	FScreenReaderReply ClearAllAnnouncements();
	FScreenReaderReply RequestSpeakWidget(const TSharedRef<IAccessibleWidget>& InWidget);

	float GetSpeechVolume() const;
	FScreenReaderReply SetSpeechVolume(float InVolume);
	float GetSpeechRate() const;
	FScreenReaderReply SetSpeechRate(float InRate);
	FScreenReaderReply MuteSpeech();
	FScreenReaderReply UnmuteSpeech();
	bool IsSpeechMuted() const;
private:
	/**
	* A callback that is bound to the text to speech delegate. The delegate fires when a string
	* is fully synthesized and played to an end userwithout any interruption or stopping.
	* This function i sused to respond to that event and speak the next queued announcement.
	*/
	void OnTextToSpeechFinishSpeaking();
	/** The current announcement that is being spoken. Will have an empty string if no announcements are currently being spoken. */
	FScreenReaderAnnouncement CurrentAnnouncement;
	/** TheA priority queue for all of the queued announcements. */
	TArray<FScreenReaderAnnouncement> AnnouncementQueue;
	TSharedRef<FTextToSpeechBase> TextToSpeech;
};
