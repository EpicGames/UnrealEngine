// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"


class GenericApplication;
class FScreenReaderBase;

/**
* The abstract interface for builders of all screen readers.
* Users should subclass and provide a concrete implementation for building their custom screen readers.
*/
class IScreenReaderBuilder
{
public:
	IScreenReaderBuilder() = default;
	virtual ~IScreenReaderBuilder() = default;

	/** The arguments to be passed to the screen reader to be built */
	struct FArgs
	{
	public:
		FArgs() = delete;
		explicit FArgs(const TSharedRef<GenericApplication>& InPlatformApplication)
			: PlatformApplication(InPlatformApplication)
		{}
		~FArgs() = default;
		/** The platform application to be passed to the screen reader for initialization. */
		const TSharedRef<GenericApplication> PlatformApplication;
	};
	/**
	* A pure virtual function to build a screen reader.
	* Users must override this class to provide logic for building their custom screen readers. 
	* @param InArgs The arguments used to build a screen reader
	* @return The constructed screen reader built from the passed in arguments 
	*/
virtual TSharedRef<FScreenReaderBase> Create(const IScreenReaderBuilder::FArgs& InArgs) = 0;
};
