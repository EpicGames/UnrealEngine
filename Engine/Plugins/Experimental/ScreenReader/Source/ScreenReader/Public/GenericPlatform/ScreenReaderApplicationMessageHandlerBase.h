// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

class FScreenReaderBase;

/**
* The base class all screen reader application message handlers should derive from. By default, this is just a proxy for the underlying target message handler.
* The screen reader application message handler will intercept all application messages and may perform processing on the events before consuming the event or passing it to the underlying target message handler.
* Child classes should override the virtual event handlers from FGenericAccessibleMessageHandler to perform input processing for thier needs.
* For example:
* A mobile screen reader application message handler can subclass this class and intercept touch inputs to perform gesture detection to provide alternative navigation schemes for users.
* A desktop screen reader application message handler can subclass this class and intercept mouse movement events to speak widgets that the user mouses over.
*/
class SCREENREADER_API FScreenReaderApplicationMessageHandlerBase : public FGenericApplicationMessageHandler
{
public:
	FScreenReaderApplicationMessageHandlerBase() = delete;
	FScreenReaderApplicationMessageHandlerBase(const TSharedRef<FGenericApplicationMessageHandler>& InTargetMessageHandler, FScreenReaderBase& InOwningScreenReader);
	virtual ~FScreenReaderApplicationMessageHandlerBase() = default;

	/** Returns true if mouse events should be processed. Else returns false. */
	bool ShouldProcessMouseInput() const { return bProcessMouseInput; }
	/** Sets whether mouse events should be intercepted and processed. If false, the mouse events will bubble straight down to the target message handler. */
	void SetProcessMouseInput(bool bInProcessMouseInput) { bProcessMouseInput = bInProcessMouseInput; }

	/** Retursn true if the key inputs should be intercepted and processed. Else returns false. */
	bool ShouldProcessKeyInput() const { return bProcessKeyInput; }
	/** Sets whether key inputs should be intercepted and processed. If set to false, the key inputs will ubble straight down to the target message handler. */
	void SetProcessKeyInput(bool bInProcessKeyInput) { bProcessKeyInput = bInProcessKeyInput; }

	/** Returns the underlying message handler that application messages bubble down to if the screen reader application message handler does not handle the event. */
	TSharedRef<FGenericApplicationMessageHandler> GetTargetMessageHandler() const { return TargetMessageHandler; }

	// FGenericApplicationMessageHandler
	// By default all the following functions just forward the call to the underlying message handler.
	// Child classes should only override functions where they want to intercept the message first and perform pre-processing or handling
	// @see TargetMessageHandler
	virtual bool ShouldProcessUserInputMessages(const TSharedPtr< FGenericWindow >& PlatformWindow) const override;
	virtual bool OnKeyChar(const TCHAR Character, const bool IsRepeat) override;
	virtual bool OnKeyDown(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat) override;
	virtual bool OnKeyUp(const int32 KeyCode, const uint32 CharacterCode, const bool IsRepeat) override;
	virtual bool OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button) override;
	virtual bool OnMouseDown(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos) override;
	virtual bool OnMouseUp(const EMouseButtons::Type Button) override;
	virtual bool OnMouseUp(const EMouseButtons::Type Button, const FVector2D CursorPos) override;
	virtual bool OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button) override;
	virtual bool OnMouseDoubleClick(const TSharedPtr< FGenericWindow >& Window, const EMouseButtons::Type Button, const FVector2D CursorPos) override;
	virtual bool OnMouseWheel(const float Delta) override;
	virtual bool OnMouseWheel(const float Delta, const FVector2D CursorPos) override;
	virtual bool OnMouseMove() override;
	virtual bool OnRawMouseMove(const int32 X, const int32 Y) override;
	virtual bool OnCursorSet() override;
	virtual bool OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue) override;
	virtual bool OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;
	virtual bool OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;
	virtual void OnBeginGesture() override;
	virtual bool OnTouchGesture(EGestureEvent GestureType, const FVector2D& Delta, float WheelDelta, bool bIsDirectionInvertedFromDevice) override;
	virtual void OnEndGesture() override;
	virtual bool OnTouchStarted(const TSharedPtr< FGenericWindow >& Window, const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;
	virtual bool OnTouchMoved(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;
	virtual bool OnTouchEnded(const FVector2D& Location, int32 TouchIndex, int32 ControllerId) override;
	virtual bool OnTouchForceChanged(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;
	virtual bool OnTouchFirstMove(const FVector2D& Location, float Force, int32 TouchIndex, int32 ControllerId) override;
	virtual void ShouldSimulateGesture(EGestureEvent Gesture, bool bEnable) override;
	virtual bool OnMotionDetected(const FVector& Tilt, const FVector& RotationRate, const FVector& Gravity, const FVector& Acceleration, int32 ControllerId) override;
	virtual bool OnSizeChanged(const TSharedRef< FGenericWindow >& Window, const int32 Width, const int32 Height, bool bWasMinimized = false) override;
	virtual void OnOSPaint(const TSharedRef<FGenericWindow>& Window) override;
	virtual FWindowSizeLimits GetSizeLimitsForWindow(const TSharedRef<FGenericWindow>& Window) const override;
	virtual void OnResizingWindow(const TSharedRef< FGenericWindow >& Window) override;
	virtual bool BeginReshapingWindow(const TSharedRef< FGenericWindow >& Window) override;
	virtual void FinishedReshapingWindow(const TSharedRef< FGenericWindow >& Window) override;
	virtual void HandleDPIScaleChanged(const TSharedRef< FGenericWindow >& Window) override;
	virtual void OnMovedWindow(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y) override;
	virtual bool OnWindowActivationChanged(const TSharedRef< FGenericWindow >& Window, const EWindowActivation ActivationType) override;
	virtual bool OnApplicationActivationChanged(const bool IsActive) override;
	virtual bool OnConvertibleLaptopModeChanged() override;
	virtual EWindowZone::Type GetWindowZoneForPoint(const TSharedRef< FGenericWindow >& Window, const int32 X, const int32 Y) override;
	virtual void OnWindowClose(const TSharedRef< FGenericWindow >& Window) override;
	virtual EDropEffect::Type OnDragEnterText(const TSharedRef< FGenericWindow >& Window, const FString& Text) override;
	virtual EDropEffect::Type OnDragEnterFiles(const TSharedRef< FGenericWindow >& Window, const TArray< FString >& Files) override;
	virtual EDropEffect::Type OnDragEnterExternal(const TSharedRef< FGenericWindow >& Window, const FString& Text, const TArray< FString >& Files) override;
	virtual EDropEffect::Type OnDragOver(const TSharedPtr< FGenericWindow >& Window) override;
	virtual void OnDragLeave(const TSharedPtr< FGenericWindow >& Window) override;
	virtual EDropEffect::Type OnDragDrop(const TSharedPtr< FGenericWindow >& Window) override;
	virtual bool OnWindowAction(const TSharedRef< FGenericWindow >& Window, const EWindowAction::Type InActionType) override;
	// 
protected:
	/** 
	* The screen reader that owns this screen reader application message handler.
	* The lifetime of the screen reader message handler follows its owning screen reader.
	* The owning screen reader can be used to retrieve screen reader users to request accessible services such as requesting nnouncements be spoken
	* or to handle accessible events.
	* @see FScreenReaderBase
	*/
	FScreenReaderBase& OwningScreenReader;
private:
	/** The underlying message handler that application messages will bubble down to if the screen reader applicaiton message handler does not consume or handle the app.application messages. */
	TSharedRef<FGenericApplicationMessageHandler> TargetMessageHandler;
	/** True if the screen reader application message handler should process mouse input. Else false. */
	bool bProcessMouseInput;
	/** True if the screen reader application message handler should process key inputs. Else false. */
	bool bProcessKeyInput;
};
