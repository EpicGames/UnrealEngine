// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

class FGenericApplicationMessageHandler;
class FScreenReaderApplicationMessageHandlerBase;
class FScreenReaderUser;
class GenericApplication;
struct FAccessibleEventArgs;


/**
* The abstract base class all screen readers must derive from.
* The screen reader sets up and tears down the screen reader framework with Activate() and Deactivate() respectively.
* When active, the screen reader will intercept all application messages, perform necessary input pre-processing and dispatch the accessible events and 
* inputs to all registered screen reader users.
* Users of derived classes must activate the screen reader to ensure the accessible inputs and accessible events are dispatched properly.
 * Derived classes must implement the pure virtual OnAccessibleEventRaised function to handle accessible events.
* @see FScreenReaderUser
*/
class SCREENREADER_API FScreenReaderBase
{
public: 
	FScreenReaderBase() = delete;
	explicit FScreenReaderBase(const TSharedRef<GenericApplication>& InPlatformApplication);
	// We disallow copying 
	FScreenReaderBase(const FScreenReaderBase& Other) = delete;
	FScreenReaderBase& operator=(const FScreenReaderBase& Other) = delete;
	virtual ~FScreenReaderBase();

	void SetScreenReaderApplicationMessageHandler(const TSharedRef<FScreenReaderApplicationMessageHandlerBase>& InScreenReaderApplicationMessageHandler) { ScreenReaderApplicationMessageHandler= InScreenReaderApplicationMessageHandler; }
	/**
	* Activates the screen reader and sets up the screen reader framework if the screen reader is inactive.
	* Once this is called, the screen reader can intercept application messages, perform input pre-processing and dispatch accessibility events to the registered screen reader users.
	* OnActivated() is called at the end of the function. Users can override the function to perform additional setup.
	* @see OnActivated()
	*/
	void Activate();
	/**
	* Deactivates the screen reader and tears down the screen reading framework if the screen reader is active.
	* The screen reader will no longer intercept application input messages and no accessible events will be dispatched to the registered screen reader users. 
	* OnDeactivated() is called at the start of this function. Users can override the function to perform additional teardown.
	* @see OnDeactivated()
	*/
	void Deactivate();
	/** Returns true if the screen reader is active. Else false. */
	bool IsActive() const { return bActive; }
	/** 
	* Registers a provided user Id to the screen reader framework and allows the user to receive and respond to accessible events and accessible input.
	* Does nothing if the passed in Id is already registered.
	* @ return Returns true if the Id is successfully registered with the screen reader. Else returns false.
	*/
	bool RegisterUser(int32 InUserId);
	/**
	* Unregisters a provided user Id from the screen reader framework and deactivates the user. The unregistered user will no longer receive or respond to accessible events and input.
	* Nothing will happen if the provided user Id has not been registered with the screen reader. 
	* @param InUserId The user Id to unregister from the screen reader framework.
	* @return Returns true if the provided user Id has been successfully unregistered. Else returns false.
	*/
	bool UnregisterUser(int32 InUserId);
	/**
	* Returns true if the passed in user Id is already registered. Else returns false.
	* @param InUserId The user Id to check for registration.
	*/
	bool IsUserRegistered(int32 InUserId) const;
	/** Unregisteres all screen reader users. No users can receive or respond to accessible events or accessible input upon calling this function. */
	void UnregisterAllUsers();
	/** 
	* Returns the screen reader user associated with the passed in Id. An assertion will trigger if the User Id is not registered. 
	* It is up to the user to ensure that the Id is valid and mapped to a valid screen reader user.
	* @param InUserId The Id of the screen reader user to be retrieved.
	* @return The screen reader user associated with the passed in Id
	*/
	TSharedRef<FScreenReaderUser> GetUserChecked(int32 InUserId) const;
	/**
	* Returns a screen reader user associated with the passed in Id if the Id is registered. Otherwise, a nullptr is returned.
	* @param InUserId The user Id of the screen reader to be retrieved.
	* @return The screen reader user associated with the Id if the user is registered. Else nullptr.
	*/
	TSharedPtr<FScreenReaderUser> GetUser(int32 InUserId) const;

protected:
	/**
	* Called at the end of Activate(). Override this function in child classes to do additional set up 
	* such as subscribing to accessible events, binding handlers for focus events or providing feedback to
	* users that the screen reader has been activated.
	*/
	virtual void OnActivate() {}
	/**
	* Called at the start of Deactivate(). Override this function in child classes to provide additional teardown such as unsubscribing from accessible events or providing
	* feedback to alert the user that the screen reader is deactivated. 
	*/
	virtual void OnDeactivate() {}
	/**
	 * The handler for all accessible events raised from FGenericAccessibleMessageHandler. Is bound to the accessible event delegate in FGenericAccessibleMessageHandler.
	 * This handler is bound during activation and unbound during deactivation.
	 * A sample implementation can be found in FSlateScreenReader.
	 * @see FSlateScreenReader
	 */
	virtual void OnAccessibleEventRaised(const FAccessibleEventArgs& Args) = 0;
	/** Returns the weak pointer to the platform application. Use this to access the accessible message handler and or retrieve cursor information. */
	TWeakPtr<GenericApplication> GetPlatformApplicationWeak() const
	{
		return PlatformApplication;
	}

private:
	/** 
	* The screen reader's application message handler. It processes input events before passing them down to the target application message handler
	* @see FScreenReaderApplicationMessageHandlerBase
	*/
	TSharedRef<FScreenReaderApplicationMessageHandlerBase> ScreenReaderApplicationMessageHandler;
	/** The platform application. Used to access the accessible message handler and swap the application message handler with the screen reader application message handler. */
	TWeakPtr<GenericApplication> PlatformApplication;
	/** True if the screen reader is active. Else false. */
	bool bActive;
};

