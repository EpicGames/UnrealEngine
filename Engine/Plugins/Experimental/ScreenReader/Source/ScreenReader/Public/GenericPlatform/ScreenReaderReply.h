// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "ScreenReaderReply.generated.h"
/**
* A struct passed around the the screen reader framework to indicate if a request has been successfully handled.
* Use this class to determine if a request to the screen reader user was successfully handled and provide user feedback accordingly.
* // @TODOAccessibility: Provide examples 
* @see FScreenReaderUser
*/
USTRUCT(BlueprintType)
struct SCREENREADER_API FScreenReaderReply
{
	GENERATED_BODY()
public:
	FScreenReaderReply() = default;
	~FScreenReaderReply() = default;

	static FScreenReaderReply Handled()
	{
		return FScreenReaderReply(true);
	}

	static FScreenReaderReply Unhandled()
	{
		return FScreenReaderReply(false);
	}

	bool IsHandled() const
	{
		return bHandled;
	}
private:
	FScreenReaderReply(bool bInHandled)
		: bHandled(bInHandled)
	{

	}
	UPROPERTY(BlueprintReadOnly, Category=ScreenReaderReply, meta=(AllowPrivateAccess = "true"))
	bool bHandled = false;
};