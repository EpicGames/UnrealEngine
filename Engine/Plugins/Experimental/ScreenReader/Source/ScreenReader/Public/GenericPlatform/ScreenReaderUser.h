// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Announcement/ScreenReaderAnnouncement.h"
#include "GenericPlatform/ScreenReaderReply.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"

class IAccessibleWidget;
class FScreenReaderAnnouncementChannel;
class IScreenReaderNavigationPolicy;

/**
* A user of the screen reader. Corresponds to a hardware device that users use.
* This class is a facade that acts as a one stop shop for all screen reading services on a per user basis.
* Multiple screen reader users can exist simultaneoulsy to facilitate local multiplayer and users can opt in or out 
* of receiving screen reader feedback by registering and unregistering with the screen reader respectively.
* A screen user is inactive by default when it is first registered through the screen reader. Users must explicitly activate the screen reader user after successful registration with the screen reader to use its services.
* Responsibilities of the class are:
* 1. Text to speech (TTS) requests - Users can request an announcement to be spoken via text to speech to  screen reader user
* 2. Accessible focus handling - A screen reader user holds information about the accessible widget it is currently focused on
 * 3. Accessible Navigation - Screen reader users have finer navigation controls around the accessible widget hierarchy as compared to regular tab navigation with keyboard or D-pad controls with a gamepad.
* @see FScreenReaderBase, FScreenReaderAnnouncement, FGenericAccessibleUser
*/
class SCREENREADER_API FScreenReaderUser : public FGenericAccessibleUser
{
public:
	explicit FScreenReaderUser(FAccessibleUserIndex InUserIndex);
	~FScreenReaderUser();
	/**
	* Requests an announcement to be spoken to the screen reader user. This is the main mechanism to provide text to speech auditory feedback
	* to users.
	*Calling this function does not guarantee that the announcement will be spoken via text to speech and be 
	* heard by a user.
	* All announcements spoken via text to speech will be asynchronous and iwll not block the game thread. 
	* If the screen reader user is active and no announcements are currently spoken, the announcement will be spoken immediately.
	* If another announcement is currently being spoken, the passed in announcement could be queued or interrupt the currently spoken announcement.
	* Examples:
	* TSharedRef<FScreenReaderUser> MyUser = ScreenReader->GetUser(MyScreenReaderUserIndex);
	* // All announcements to be spoken should be localized to provide language support for the text to speech system.
	* static const FText MyText = LOCTEXT("ExampleUserFeedback", "Feedback to user.");
	* // This makes the announcement interrupt any currently spoken announcement with lower priority and is played immediately. The announcement will be uninterruptable and guarantees the user will hear the announcement.
	* // Use this sparingly for only critical information that the user needs to hear such as registration to the screen reader or important in game events.
	* MyUser->RequestSpeak(FScreenReaderAnnouncement(MyText.ToString(), FScreenReaderAnnouncementInfo::Important()));
	* // This makes the announcement a feedback for a user action or event. It is interruptable.
	* // Use this to alert users to an action they performed 
	* MyUser->RequestSpeak(FScreenReaderAnnouncement(MyText.ToString(), FScreenReaderAnnouncementInfo::UserFeedback()));
	* @param InAnnouncement - The announcement requested to be spoken to the screen reader user
	* @return FScreenReaderReply::Handled() if the request was successfully processed. Else return FScreenReader::Unhandled()
	* @see FScreenReaderAnnouncement, FScreenReaderAnnouncementInfo
	*/
	FScreenReaderReply RequestSpeak(FScreenReaderAnnouncement InAnnouncement);
	/**
	* Immediately stops speaking any currently spoken announcement.
	* Does nothing if there is no announcement currently being spoken. 
	* @return FScreenReader::Handled() if the request to stop speaking is successfully processed. Else returns FScreenReaderReply::Unhandled()
	*/
	FScreenReaderReply StopSpeaking();
	/** Returns true if an announcement is currently being spoken. Else returns false.*/
	bool IsSpeaking() const;
	/**
	* Requests the information about an accessibility widget to be read out to a user.
	* The same guarantees about the announcement being spoken in RequestSpeak() apply for this function.
	* @param InWidget The widget to ahve its contents spoken to a user.
	* @return FScreenReaderReply::Handled() if the widget's information is successfully spoken. Else returns FScreenReaderReply::Unhandled();
	*/
	FScreenReaderReply RequestSpeakWidget(const TSharedRef<IAccessibleWidget>& InWidget);
	/** Returns the speech volume for text to speech. Value will be between 0.0f and 1.0f. */
	float GetSpeechVolume() const;
	/**
	* Sets the speech volume for text to speech to speak at. The passed in value will be clamped between 0.0f and 1.0f.
	* @return FScreenReaderReply::Handled() if the volume is successfully set. Else FScreenReader::Unhandled() is returned.
	*/
	FScreenReaderReply SetSpeechVolume(float InVolume);
	/** Returns the speech rate text to speech is speaking at for this user. Value will be between 0.0f and 1.0f. */
	float GetSpeechRate() const;
	/**
	* Sets the speech rate text to speech will speak at for this user. Passed in value is clamped between 0.0f and 1.0f.
	* @return FScreenReaderReply::Handled() if the speech rate is successfully set. Else, FScreenReaderReply::Unhandled() is returned.
	*/
	FScreenReaderReply SetSpeechRate(float InRate);
	/**
	* Mutes the text to speech for this user so requests to speak strings will be inaudible.
	* @return FScreenReaderReply::Handled() if the text to speech is successfully muted. Else, FScreenReaderReply::Unhandled() is returned.
	*/
	FScreenReaderReply MuteSpeech();
	/**
	* Unmutes the text to speech for this user so that requests to speak strings will be audible again.
	* @return FScreenReaderReply::Handled() if the text to speech si successfully unmuted. Else, FScreenReaderReply::Unhandled() is returned.
	*/
	FScreenReaderReply UnmuteSpeech();
	/** Returns true if the text to speech for this user is muted. Else returns false. */
	bool IsSpeechMuted() const;

	
	// Navigation
	/** Sets the navigation policy this screen reader user will be using. The screen reader navigation policy will affect all functionality to do with shifting focus to the next available widget. */
	void SetNavigationPolicy(const TSharedRef<IScreenReaderNavigationPolicy>& InNavigationPolicy);
	/**
	 * Shifts focus to the next sibling from the user's currently focused accessible widget based on the current navigation policy.
	 * @return FScreenReaderReply::Handled() if focus has successfully shifted to a next sibling from the currently focused accessible widget. Else returns FScreenReaderReply::Unhandled()
	 */
	FScreenReaderReply NavigateToNextSibling();
	/**
	 * Shifts focus to the previous sibling from the user's currently focused accessible widget based on the current navigation policy.
	 * @return FScreenReaderReply::Handled() if focus has successfully shifted to a previous sibling from the currently focused accessible widget. Else returns FScreenReaderReply::Unhandled()
	 */
	FScreenReaderReply NavigateToPreviousSibling();
	/**
	 * Shifts focus to the first ancestor of the user's currently focused accessible widget based on the current navigation policy.
	 * @return FScreenReaderReply::Handled() if focus has successfully shifted to the first ancestor of the currently focused accessible widget. Else returns FScreenReaderReply::Unhandled()
	 */
	FScreenReaderReply NavigateToFirstAncestor();
	/**
	 * Shifts focus to the first child from the user's currently focused accessible widget based on the current navigation policy.
	 * Focus is unaffected if the user's currently focused accessible widget has no children.
	 * @return FScreenReaderReply::Handled() if focus has successfully shifted to the first child from the currently focused accessible widget based on the navigation policy. Else returns FScreenReaderReply::Unhandled()
	 */
	FScreenReaderReply NavigateToFirstChild();
	/**
	 * Shifts focus to the logical next widget in the accessible widget hierarchy from the user's currently focused accessible widget based on the current navigation policy.
	 * See IAccessibleWidget::GetNextWidgetInHierarchy for an explanation of what the logical next widget in the accessible hierarchy means.
	 * @return FScreenReaderReply::Handled() if focus has successfully shifted to a next widget from the currently focused accessible widget. Else returns FScreenReaderReply::Unhandled()
	 */
	FScreenReaderReply NavigateToNextWidgetInHierarchy();
	/**
	 * Shifts focus to the logical previous widget in the accessible widget hierarchy from the user's currently focused accessible widget based on the current navigation policy.
	 * See IAccessibleWidget::GetPreviousWidgetInHierarchy for an explanation of what the logical previous widget in the accessible hierarchy means.
	 * @return FScreenReaderReply::Handled() if focus has successfully shifted to a logical previous widget from the currently focused accessible widget. Else returns FScreenReaderReply::Unhandled()
	 */
	FScreenReaderReply NavigateToPreviousWidgetInHierarchy();
	/**
	* Activates the screen reader user and fulfill requests for accessibility services such as text to speech that clients can make.
	* When screen reader users are firstr registered with a screen reader, they are deactivated by default. Users must explicitly activate the screen reader user.
	*/
	void Activate();
	/** 
	* Deactivates the screen reader and disables all announcement and text to speech services
	* making them do nothing.
	*/
	void Deactivate();
	/** Returns true if the screen reader user is active. Else returns false.*/
	bool IsActive() const 
	{ 
		return bActive; 
	}

protected:
	//~ Begin FGenericAccessibleUser interface
	virtual void OnUnregistered() override;
	//~ End FGenericAccessibleUser interface
	
private:
	/** Responsible for handling all incoming announcement requests and speaking them via text to speech if possible */
	TUniquePtr<FScreenReaderAnnouncementChannel> AnnouncementChannel;
	TSharedRef<IScreenReaderNavigationPolicy> NavigationPolicy;
	bool bActive;
};

