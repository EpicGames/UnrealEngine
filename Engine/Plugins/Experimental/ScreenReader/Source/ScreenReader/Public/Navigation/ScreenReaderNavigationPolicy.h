// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"

/**
 * An interface that controls how a screen reader user navigates around the accessible widget hierarchy and what accessible widgets can
 * be navigated to from a source widget.
 * Clients can provide custom implementations to provide different navigation behaviors for screen reader users at runtime. A sample implementation is FDefaultScreenReaderNavigationPolicy.
 * @see FScreenReaderUser, FDefaultScreenReaderNavigaitonPolicy
 */
class SCREENREADER_API IScreenReaderNavigationPolicy
{
public:
	virtual ~IScreenReaderNavigationPolicy() = default;
	/** Returns the first instance of a next sibling from the source widget that satisfies the navigation policy. */
	virtual TSharedPtr<IAccessibleWidget> GetNextSiblingFrom(const TSharedRef<IAccessibleWidget>& Source) const = 0;
	/** Returns the first instance of a previous sibling from the source widget that satisfies the navigation policy. */
	virtual TSharedPtr<IAccessibleWidget> GetPreviousSiblingFrom(const TSharedRef<IAccessibleWidget>& Source) const = 0;
	/** Returns the first ancestor from the source widget that satisfies the navigation policy. */
	virtual TSharedPtr<IAccessibleWidget> GetFirstAncestorFrom(const TSharedRef<IAccessibleWidget>& Source) const = 0;
	/** Returns the first child from the source widget that satisfies the navigation policy. */
	virtual TSharedPtr<IAccessibleWidget> GetFirstChildFrom(const TSharedRef<IAccessibleWidget>& Source) const = 0;
	/**
	 * Returns the first instance of a logical next widget from the source widget that satisfies the navigation policy.
	 * See IAccessibleWidget::GetNextWidgetInHierarchy for an explanation of what the logical next widget in the accessible hierarchy means.
	 * @see IAccessibleWidget
	 */
	virtual TSharedPtr<IAccessibleWidget> GetNextWidgetInHierarchyFrom(const TSharedRef<IAccessibleWidget>& Source) const = 0;
	/**
	 * Returns the first instance of a logical previous widget from the source widget that satisfies the navigation policy.
	 * See IAccessibleWidget::GetPreviousWidgetInHierarchy for an explanation of what the logical previous widget in the accessible hierarchy means.
	 * @see IAccessibleWidget
	 */
	virtual TSharedPtr<IAccessibleWidget> GetPreviousWidgetInHierarchyFrom(const TSharedRef<IAccessibleWidget>& Source) const = 0;
};

class SCREENREADER_API FScreenReaderDefaultNavigationPolicy final : public IScreenReaderNavigationPolicy
{
public:
	struct FScreenReaderDefaultNavigationPredicate
	{
		bool operator()(const TSharedRef<IAccessibleWidget>& InWidget) const
		{
			return InWidget->CanCurrentlyAcceptAccessibleFocus();
		}
	};
	virtual ~FScreenReaderDefaultNavigationPolicy() = default;
	//~ Begin IScreenReaderNavigationPolicy Interface
	virtual TSharedPtr<IAccessibleWidget> GetNextSiblingFrom(const TSharedRef<IAccessibleWidget>& Source) const override final
	{
		return IAccessibleWidget::SearchForNextSiblingFrom(Source, FScreenReaderDefaultNavigationPredicate());
	}
	virtual TSharedPtr<IAccessibleWidget> GetPreviousSiblingFrom(const TSharedRef<IAccessibleWidget>& Source) const override final
	{
		return IAccessibleWidget::SearchForPreviousSiblingFrom(Source, FScreenReaderDefaultNavigationPredicate());
	}
	virtual TSharedPtr<IAccessibleWidget> GetFirstAncestorFrom(const TSharedRef<IAccessibleWidget>& Source) const override final
	{
		return IAccessibleWidget::SearchForAncestorFrom(Source, FScreenReaderDefaultNavigationPredicate());
	}
	virtual TSharedPtr<IAccessibleWidget> GetFirstChildFrom(const TSharedRef<IAccessibleWidget>& Source) const override final
	{
		return IAccessibleWidget::SearchForFirstChildFrom(Source, FScreenReaderDefaultNavigationPredicate());
	}
	virtual TSharedPtr<IAccessibleWidget> GetNextWidgetInHierarchyFrom(const TSharedRef<IAccessibleWidget>& Source) const override final
	{
		return IAccessibleWidget::SearchForNextWidgetInHierarchyFrom(Source, FScreenReaderDefaultNavigationPredicate());
	}
	virtual TSharedPtr<IAccessibleWidget> GetPreviousWidgetInHierarchyFrom(const TSharedRef<IAccessibleWidget>& Source) const override final
	{
		return IAccessibleWidget::SearchForPreviousWidgetInHierarchyFrom(Source, FScreenReaderDefaultNavigationPredicate());
	}
	//~ End IScreenReaderNavigationPolicy Interface
};

