// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;


public class ScreenReader : ModuleRules
{
	public ScreenReader(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
			"Core",
			"CoreUObject"
		});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
			"ApplicationCore",
			"TextToSpeech",
			"Engine"
		});
	}
}
